{
	"FileVersion": 1,
	"FriendlyName": "Security Sandbox",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "Provides features to help reduce the operating system permissions your game client runs with and therefore reduce the impact to players if an attacker takes control of it through a vulnerability.",
	"Category": "Security",
	"EnabledByDefault": false,
	"IsExperimentalVersion": true,
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SecuritySandbox",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}