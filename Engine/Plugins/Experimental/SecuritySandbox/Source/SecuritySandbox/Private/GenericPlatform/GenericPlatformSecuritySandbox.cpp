// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatformSecuritySandbox.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"

void FGenericPlatformSecuritySandbox::Init()
{
	bIsSandboxEnabled = InitIsSandboxEnabled();

	// Always bind this delegate if the plugin is enabled. When invoked it will check config to determine whether it needs to actually do anything.
	if (IsEnabled())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.AddThreadSafeSP(AsShared(), &FGenericPlatformSecuritySandbox::OnEngineInitComplete);
	}
}

void FGenericPlatformSecuritySandbox::RestrictSelf()
{
	// Validate config and conditions and only call PlatformRestrictSelf if all looks good.
	if (!IsEnabled())
	{
		UE_LOGF(LogSecuritySandbox, Warning, "SecuritySandbox is disabled by config or command line param. Skipping self restrictions.");
		return;
	}

	if (ensureMsgf(!bHasRestrictedSelf, TEXT("RestrictSelf must be called exactly once.")))
	{
		UE_LOGF(LogSecuritySandbox, Log, "Restricting self.");
		PlatformRestrictSelf();
		bHasRestrictedSelf = true;
	}
}

bool FGenericPlatformSecuritySandbox::IsEnabled()
{
	return bIsSandboxEnabled;
}

void FGenericPlatformSecuritySandbox::OnEngineInitComplete()
{
	const USecuritySandboxSettings* Settings = GetDefault<USecuritySandboxSettings>();
	if (Settings && Settings->bAutoRestrictSelf)
	{
		UE_LOGF(LogSecuritySandbox, Verbose, "Engine init complete - triggering RestrictSelf.");
		RestrictSelf();
	}
}

bool FGenericPlatformSecuritySandbox::InitIsSandboxEnabled()
{
	// Initial value is based on the plugin settings configuration.
	const USecuritySandboxSettings* Settings = GetDefault<USecuritySandboxSettings>();
	bool bEnabled = (Settings && Settings->bIsEnabledByDefault);

	// Allow command line override.
	// -NoSecuritySandbox: Explicitly disable.
	// -WithSecuritySandbox: Explicitly enable.
	// If both are specified then WithSecuritySandbox wins.
	bEnabled = (bEnabled && !FParse::Param(FCommandLine::Get(), TEXT("NoSecuritySandbox")));
	bEnabled = (bEnabled || FParse::Param(FCommandLine::Get(), TEXT("WithSecuritySandbox")));

	switch (FApp::GetBuildTargetType())
	{
		// Client and Game are supported as the main targets for SecuritySandbox.
		case EBuildTargetType::Client:
		case EBuildTargetType::Game:
			break;

		// Editor is normally disabled but may be enabled by a config flag for testing.
		case EBuildTargetType::Editor:
			bEnabled = (bEnabled && Settings && Settings->bAllowSandboxInEditor);
			break;

		// Other targets are not currently supported.
		default:
			bEnabled = false;
			break;
	}
	return bEnabled;
}
