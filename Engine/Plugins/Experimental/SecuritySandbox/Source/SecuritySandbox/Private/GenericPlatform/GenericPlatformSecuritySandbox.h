// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ISecuritySandbox.h"
#include "SecuritySandboxSettings.h"

class FGenericPlatformSecuritySandbox : public ISecuritySandbox, public TSharedFromThis<FGenericPlatformSecuritySandbox>
{
public:
	virtual ~FGenericPlatformSecuritySandbox() {};
	FGenericPlatformSecuritySandbox() {}

	// Initialize the sandbox, will be called exactly once.
	virtual void Init();

	// Public interface function from ISecuritySandbox that user code can call without validation.
	virtual void RestrictSelf() override final;

	// True if restriction can be attempted. If false, RestrictSelf does nothing.
	virtual bool IsEnabled() override final;
	
protected:
	// Implementation function for derived platform classes which we guarantee will be called at most once with configuration options taken into consideration.
	virtual void PlatformRestrictSelf() {};

private:
	// True if restriction can be attempted. If false, RestrictSelf does nothing.
	bool bIsSandboxEnabled = false;
	
	// True if RestrictSelf has run. This doesn't convey success or failure, just that it was attempted.
	bool bHasRestrictedSelf = false;

	// Set the initial value of bIsSandboxEnabled based on config and command line.
	bool InitIsSandboxEnabled();

	// Used for automatic RestrictSelf if configured in the plugin's settings.
	void OnEngineInitComplete();
};
