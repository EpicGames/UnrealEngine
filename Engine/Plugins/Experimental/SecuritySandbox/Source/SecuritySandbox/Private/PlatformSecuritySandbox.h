// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if PLATFORM_WINDOWS
#include "Windows/WindowsPlatformSecuritySandbox.h"
typedef FWindowsPlatformSecuritySandbox FPlatformSecuritySandbox;
#else
#include "GenericPlatform/GenericPlatformSecuritySandbox.h"
typedef FGenericPlatformSecuritySandbox FPlatformSecuritySandbox;
#endif