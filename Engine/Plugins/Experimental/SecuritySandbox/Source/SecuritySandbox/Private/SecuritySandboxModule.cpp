// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISecuritySandboxModule.h"
#include "PlatformSecuritySandbox.h"
#include "Templates/SharedPointer.h"

DEFINE_LOG_CATEGORY(LogSecuritySandbox);

class FSecuritySandboxModule : public ISecuritySandboxModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void RestrictSelf() override;
	virtual bool IsEnabled() override;
	
private:
	TSharedPtr<FPlatformSecuritySandbox> PlatformSecuritySandbox;
};

void FSecuritySandboxModule::StartupModule()
{
	if (!PlatformSecuritySandbox)
	{
		PlatformSecuritySandbox = MakeShared<FPlatformSecuritySandbox>();
		PlatformSecuritySandbox->Init();
	}
}

void FSecuritySandboxModule::ShutdownModule()
{
	PlatformSecuritySandbox = nullptr;
}

void FSecuritySandboxModule::RestrictSelf()
{
	if (!PlatformSecuritySandbox)
	{
		// This should never happen.
		UE_LOGF(LogSecuritySandbox, Error, "RestrictSelf called with no PlatformSecuritySandbox instance available");
		checkNoEntry();
		return;
	}

	PlatformSecuritySandbox->RestrictSelf();
}

bool FSecuritySandboxModule::IsEnabled()
{
	return PlatformSecuritySandbox->IsEnabled();
}

IMPLEMENT_MODULE(FSecuritySandboxModule, SecuritySandbox)
