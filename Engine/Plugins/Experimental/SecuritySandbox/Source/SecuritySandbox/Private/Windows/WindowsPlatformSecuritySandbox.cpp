// Copyright Epic Games, Inc. All Rights Reserved.
#include "WindowsPlatformSecuritySandbox.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Windows/WindowsApplication.h"

// Needed for opening URLs via WinRT APIs when restricted.
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <wrl\client.h>
	#include <wrl\wrappers\corewrappers.h>
	#include <Windows.System.h>
	#include <Windows.Foundation.h>
	#include <processthreadsapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#pragma comment(lib, "windowsapp.lib")

#define UE_LOG_WINAPIFAIL(Verbosity, ApiName) UE_LOG(LogSecuritySandbox, Verbosity, ApiName TEXT(" failed with %i"), GetLastError())

struct SidContainer
{
	SidContainer() : Sid(reinterpret_cast<SID*>(Buffer)), Size(sizeof(Buffer))
	{
		FMemory::Memzero(Buffer);
	}
	SID *const Sid;
	const size_t Size;

private:
	uint8 Buffer[SECURITY_MAX_SID_SIZE];
};

void FWindowsPlatformSecuritySandbox::PlatformRestrictSelf()
{
	ApplyJobRestrictions();
	SetProcessMitigations();
	SetLowIntegrityLevel();

	if (bNeedsCustomHandlerForURL)
	{
		if (ensure(!FCoreDelegates::LaunchCustomHandlerForURL.IsBound()))
		{
			FCoreDelegates::LaunchCustomHandlerForURL.BindThreadSafeSP(SharedThis(this), &FWindowsPlatformSecuritySandbox::CustomHandlerForURL);
			UE_LOGF(LogSecuritySandbox, Verbose, "Registered FCoreDelegates::LaunchCustomHandlerForURL");
		}
		else
		{
			UE_LOGF(LogSecuritySandbox, Warning, "FCoreDelegates::LaunchCustomHandlerForURL is already bound. Skipping.");
		}
	}
}

void FWindowsPlatformSecuritySandbox::SetLowIntegrityLevel()
{
	const USecuritySandboxSettings* Settings = GetDefault<USecuritySandboxSettings>();
	if (Settings)
	{
		if (Settings->bUseLowIntegrityLevel)
		{
			if (FPaths::ShouldSaveToUserDir())
			{
				// Make sure that FWindowsPlatformProcess is configured for low integrity mode. Otherwise settings, logs, temporary files, etc will most likely have been
				// created in directories that we will lose write access to causing all sorts of problems. The FWindowsPlatformProcess integrity config can be set via a
				// preprocessor flag in Target.cs for engine source builds or a command line parameter at runtime.
				ensureMsgf(FWindowsPlatformProcess::ShouldExpectLowIntegrityLevel(), TEXT("Setting low integrity level but FWindowsPlatformProcess is not configured for it. Pass -ExpectLowIntegrityLevel or update Target.cs."));

				// Open our current process token and irreversibly assign it to low integrity level.
				HANDLE hProcessToken = INVALID_HANDLE_VALUE;
				if (OpenProcessToken(GetCurrentProcess(), TOKEN_ADJUST_DEFAULT, &hProcessToken))
				{
					SidContainer Sid = {};
					DWORD SidSize = Sid.Size;
					if (CreateWellKnownSid(WinLowLabelSid, nullptr, Sid.Sid, &SidSize))
					{
						TOKEN_MANDATORY_LABEL LowIntegrity = {};
						LowIntegrity.Label.Sid = Sid.Sid;
						if (SetTokenInformation(hProcessToken, TokenIntegrityLevel, &LowIntegrity, sizeof(LowIntegrity)))
						{
							UE_LOGF(LogSecuritySandbox, Log, "Set low integrity level");

							// Opening URLs via ShellExecute or CreateProcess fails at low integrity level so we need a custom handler.
							bNeedsCustomHandlerForURL = true;
						}
						else
						{
							UE_LOG_WINAPIFAIL(Warning, TEXT("SetTokenInformation"));
						}
					}
					else
					{
						UE_LOG_WINAPIFAIL(Warning, TEXT("CreateWellKnownSid"));
					}
					CloseHandle(hProcessToken);
				}
				else
				{
					UE_LOG_WINAPIFAIL(Warning, TEXT("OpenProcessToken"));
				}
			}
			else
			{
				// Windows low integrity denies write access to almost all locations except for <User>\AppData\LocalLow, so we can only do it when configured to write to the user directory.
#if UE_BUILD_SHIPPING
				UE_LOGF(LogSecuritySandbox, Error, "Low integrity level is configured but ShouldSaveToUserDir returned false. Low integrity level will not be set.");
#else
				UE_LOGF(LogSecuritySandbox, Log, "Low integrity level is configured but ShouldSaveToUserDir returned false. Low integrity level will not be set. "
					"This is normal for Development and Test builds. If you want to test with low integrity, pass -SaveToUserDir on the command line.");
#endif
			}
		}
		else
		{
			UE_LOGF(LogSecuritySandbox, Log, "Low integrity level disabled by config, skipping");
		}
	}
	else 
	{
		UE_LOGF(LogSecuritySandbox, Warning, "Config is not available, skipping integrity level change");
	}
}

void FWindowsPlatformSecuritySandbox::SetProcessMitigations()
{
	// These functions were introduced with Windows 8. The UE min spec is Windows 10, but at the time of writing _WIN32_WINNT is still
	// set to _WIN32_WINNT_WIN7 and changing it just for this would be disruptive, so we can fetch them dynamically instead.
	typedef bool(WINAPI*pfnGetProcessMitigationPolicy)(HANDLE hProcess, PROCESS_MITIGATION_POLICY MitigationPolicy, PVOID lpBuffer, SIZE_T dwLength);
	typedef bool(WINAPI*pfnSetProcessMitigationPolicy)(PROCESS_MITIGATION_POLICY MitigationPolicy, PVOID lpBuffer, SIZE_T dwLength);

	const USecuritySandboxSettings* Settings = GetDefault<USecuritySandboxSettings>();
	if (Settings)
	{
		if (Settings->bDisallowLowIntegrityLibraries)
		{
			HMODULE hKernel32 = nullptr;
			if (GetModuleHandleEx(0, TEXT("kernel32.dll"), &hKernel32))
			{
				const pfnGetProcessMitigationPolicy fnGetProcessMitigationPolicy = reinterpret_cast<pfnGetProcessMitigationPolicy>((void*)GetProcAddress(hKernel32, "GetProcessMitigationPolicy"));
				const pfnSetProcessMitigationPolicy fnSetProcessMitigationPolicy = reinterpret_cast<pfnSetProcessMitigationPolicy>((void*)GetProcAddress(hKernel32, "SetProcessMitigationPolicy"));
				if (fnGetProcessMitigationPolicy && fnSetProcessMitigationPolicy)
				{
					PROCESS_MITIGATION_IMAGE_LOAD_POLICY ImageLoadPolicy = {};
					if (fnGetProcessMitigationPolicy(GetCurrentProcess(), ProcessImageLoadPolicy, &ImageLoadPolicy, sizeof(ImageLoadPolicy)))
					{
						if (ImageLoadPolicy.NoLowMandatoryLabelImages != 1)
						{
							// Prevent low integrity images from being loaded. This would e.g. block an attacker who has hijacked
							// the game after it dropped to low integrity from simply writing a DLL file to disk and calling LoadLibrary.
							ImageLoadPolicy.NoLowMandatoryLabelImages = 1;
							if (fnSetProcessMitigationPolicy(ProcessImageLoadPolicy, &ImageLoadPolicy, sizeof(ImageLoadPolicy)))
							{
								UE_LOGF(LogSecuritySandbox, Log, "Set process mitigations");
							}
							else
							{
								UE_LOG_WINAPIFAIL(Warning, TEXT("SetProcessMitigationPolicy"));
							}
						}
						else
						{
							UE_LOGF(LogSecuritySandbox, Verbose, "NoLowMandatoryLabelImages already set, skipping");
						}
					}
					else
					{
						UE_LOG_WINAPIFAIL(Warning, TEXT("GetProcessMitigationPolicy"));
					}
				}
				else
				{
					UE_LOGF(LogSecuritySandbox, Error, "Failed to resolve process mitigation exports");
				}
				FreeLibrary(hKernel32);
			}
			else
			{
				UE_LOGF(LogSecuritySandbox, Error, "Failed to find kernel32.dll");
			}
		}
		else
		{
			UE_LOGF(LogSecuritySandbox, Verbose, "Process mitigations disabled by config, skipping");
		}
	}
	else
	{
		UE_LOGF(LogSecuritySandbox, Warning, "Config is not available, skipping process mitigations");
	}
}

void FWindowsPlatformSecuritySandbox::ApplyJobRestrictions()
{
	const USecuritySandboxSettings* Settings = GetDefault<USecuritySandboxSettings>();
	if (Settings)
	{
		if (Settings->bDisallowChildProcesses || Settings->bDisallowSystemOperations)
		{
			HANDLE hJob = CreateJobObject(nullptr, nullptr);
			if (hJob)
			{
				// Prevent creation of child processes.
				if (Settings->bDisallowChildProcesses)
				{
					JOBOBJECT_BASIC_LIMIT_INFORMATION BasicInfo = {};
					BasicInfo.ActiveProcessLimit = 1;
					BasicInfo.LimitFlags = JOB_OBJECT_LIMIT_ACTIVE_PROCESS;
					if (SetInformationJobObject(hJob, JobObjectBasicLimitInformation, &BasicInfo, sizeof(BasicInfo)))
					{
						UE_LOGF(LogSecuritySandbox, Log, "Set job basic limits: 0x%X", BasicInfo.LimitFlags);

						// UE tries to CreateProcess by default when launching a URL to e.g. open the browser, which will fail if we block child processes.
						bNeedsCustomHandlerForURL = true;
					}
					else
					{
						UE_LOG_WINAPIFAIL(Warning, TEXT("SetInformationJobObject Basic"));
					}
				}

				// Prevent unnecessary system operations.
				if (Settings->bDisallowSystemOperations)
				{
					JOBOBJECT_BASIC_UI_RESTRICTIONS BasicUi = {};
					BasicUi.UIRestrictionsClass = JOB_OBJECT_UILIMIT_EXITWINDOWS | JOB_OBJECT_UILIMIT_GLOBALATOMS | JOB_OBJECT_UILIMIT_HANDLES | JOB_OBJECT_UILIMIT_SYSTEMPARAMETERS;
					if (SetInformationJobObject(hJob, JobObjectBasicUIRestrictions, &BasicUi, sizeof(BasicUi)))
					{
						UE_LOGF(LogSecuritySandbox, Log, "Set job UI limits: 0x%X", BasicUi.UIRestrictionsClass);
					}
					else
					{
						UE_LOG_WINAPIFAIL(Warning, TEXT("SetInformationJobObject UI"));
					}
				}

				// Irreversibly assign ourselves to the job object and close the handle.
				if (!AssignProcessToJobObject(hJob, GetCurrentProcess()))
				{
					UE_LOG_WINAPIFAIL(Warning, TEXT("AssignProcessToJobObject"));
				}
				CloseHandle(hJob);
			}
			else
			{
				UE_LOG_WINAPIFAIL(Warning, TEXT("CreateJobObject"));
			}
		}
		else
		{
			UE_LOGF(LogSecuritySandbox, Verbose, "Job restrictions disabled by config, skipping");
		}
	}
	else
	{
		UE_LOGF(LogSecuritySandbox, Warning, "Config is not available, skipping job restrictions");
	}
}

void FWindowsPlatformSecuritySandbox::CustomHandlerForURL(const FString& URL, FString* ErrorMsg)
{
	using namespace Microsoft::WRL;
	using namespace Microsoft::WRL::Wrappers;
	using namespace ABI::Windows::System;
	using namespace ABI::Windows::Foundation;

	UE_LOGF(LogSecuritySandbox, Log, "CustomHandlerForURL %ls", *URL);

	FString LocalError;
	if (FWindowsPlatformMisc::CoInitialize())
	{
	    ComPtr<ILauncherStatics> launcherStatics;
	    HRESULT hr = GetActivationFactory(HStringReference(RuntimeClass_Windows_System_Launcher).Get(), &launcherStatics);   
	    if (SUCCEEDED(hr))
	    {
	        ComPtr<IUriRuntimeClassFactory> uriFactory;
	        hr = GetActivationFactory(HStringReference(RuntimeClass_Windows_Foundation_Uri).Get(), &uriFactory);
	        if (SUCCEEDED(hr))
	        {
	            HString uriHString;
	            hr = uriHString.Set(*URL);
	            if (SUCCEEDED(hr))
	            {
	                ComPtr<IUriRuntimeClass> uri;
	                hr = uriFactory->CreateUri(uriHString.Get(), &uri);
	                if (SUCCEEDED(hr))
	                {
	                    ComPtr<IAsyncOperation<bool>> asyncOp;
	                    hr = launcherStatics->LaunchUriAsync(uri.Get(), &asyncOp);
	                    if (!SUCCEEDED(hr)) { LocalError = FString::Printf(TEXT("LaunchUriAsync failed (HRESULT: 0x%08X)"), hr); }
	                }
					else
					{
						LocalError = FString::Printf(TEXT("Failed to create Uri (HRESULT: 0x%08X)"), hr);
					}
	            }
				else
				{
					LocalError = FString::Printf(TEXT("Failed to set HString (HRESULT: 0x%08X)"), hr);
				}
	        }
			else
			{
				LocalError = FString::Printf(TEXT("Failed to get IUriRuntimeClassFactory (HRESULT: 0x%08X)"), hr);
			}
	    }
		else
		{
			LocalError = FString::Printf(TEXT("Failed to get ILauncherStatics (HRESULT: 0x%08X)"), hr);
		}
		FWindowsPlatformMisc::CoUninitialize();
	}
	else
	{
		LocalError = TEXT("CoInitialize failed");
	}

	if (!LocalError.IsEmpty())
	{
		UE_LOGF(LogSecuritySandbox, Warning, "CustomHandlerForURL failed: %ls", *LocalError);
	}
	if (ErrorMsg) { *ErrorMsg = LocalError; }
}

#undef UE_LOG_WINAPIFAIL