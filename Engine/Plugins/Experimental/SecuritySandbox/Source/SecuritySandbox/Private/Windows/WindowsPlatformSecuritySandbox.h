// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GenericPlatform/GenericPlatformSecuritySandbox.h"

class FWindowsPlatformSecuritySandbox : public FGenericPlatformSecuritySandbox
{
public:
	virtual ~FWindowsPlatformSecuritySandbox() {}
	virtual void PlatformRestrictSelf() override;

private:
	void SetLowIntegrityLevel();
	void SetProcessMitigations();
	void ApplyJobRestrictions();

	void CustomHandlerForURL(const FString& URL, FString* ErrorMsg);

	typedef FGenericPlatformSecuritySandbox Super;

	bool bNeedsCustomHandlerForURL = false;
};
