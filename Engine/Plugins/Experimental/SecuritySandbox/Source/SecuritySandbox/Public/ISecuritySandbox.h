// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSecuritySandbox, Log, All);

class ISecuritySandbox
{
public:
	/**
	 * Permanently restrict this process.
	 * The exact restrictions applied depend on operating system and configuration.
	 * 
	 * Your game can initialize with normal operating system access then call this function before
	 * activating online features or loading user generated content to reduce risks. USecuritySandboxSettings
	 * can also be configured to automatically apply restrictions without needing to call this function.
	 * If SecuritySandbox is disabled by config or command line parameter then this function will do nothing.
	 * 
	 * NOTE: State initialized/acquired prior to this call may allow bypassing restrictions,
	 * e.g. on Windows file handles that were opened beforehand and not closed can still be used.
	 */
	virtual void RestrictSelf() = 0;

	/**
	 * Check if SecuritySandbox as a whole is enabled. If this returns false then
	 * it may be disabled due to build configuration or command line override.
	 * Individual components may still be disabled in USecuritySandboxSettings.
	 * 
	 * @return True if SecuritySandbox is enabled, otherwise false
	 */
	virtual bool IsEnabled() = 0;
};
