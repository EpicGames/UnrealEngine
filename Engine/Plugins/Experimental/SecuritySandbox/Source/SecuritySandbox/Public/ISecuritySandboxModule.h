// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleManager.h"
#include "ISecuritySandbox.h"

/**
 * Provides features to help minimize the operating system permissions your game client runs with and therefore minimize the impact to players if an attacker takes control of it through a vulnerability.
 * See ISecuritySandbox for the API.
 */
class ISecuritySandboxModule : public IModuleInterface, public ISecuritySandbox
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ISecuritySandboxModule& Get()
	{
		// Check whether the module is already loaded first. This allows Get() to be called on non-game threads.
		// LoadModuleChecked is not non-game-thread-safe.
		ISecuritySandboxModule* ModulePtr = FModuleManager::GetModulePtr<ISecuritySandboxModule>("SecuritySandbox");
		if (ModulePtr)
		{
			return *ModulePtr;
		}

		return FModuleManager::LoadModuleChecked<ISecuritySandboxModule>("SecuritySandbox");
	}

	/**
	 * Checks to see if this module is loaded and ready. It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("SecuritySandbox");
	}

	/**
	 * Helper for calling ISecuritySandbox::IsEnabled. See that function for details.
	 *
	 * @return True if SecuritySandbox is enabled, otherwise false.
	 */
	static inline bool IsSandboxEnabled()
	{
		return IsAvailable() && Get().IsEnabled();
	}
};
