// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "SecuritySandboxSettings.generated.h"

/**
 * Implements settings for the Security Sandbox plugin. These are visible in the Project Settings section of the editor.
 */
UCLASS(config = Engine, defaultconfig, meta=(DisplayName="Security Sandbox"))
class SECURITYSANDBOX_API USecuritySandboxSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USecuritySandboxSettings() {};
	
	virtual FName GetCategoryName() const override { return FName(TEXT("Plugins")); }

	/** If true, no command line arguments are needed to activate the sandbox features. If false, -WithSecuritySandbox must be passed to activate them. */
	UPROPERTY(EditAnywhere, config, Category = General)
	bool bIsEnabledByDefault = true;

	/** If true, the game will automatically restrict itself after engine initialization completes. Otherwise you must explicitly call ISecuritySandboxModule::RestrictSelf at an appropriate time. */
	UPROPERTY(EditAnywhere, config, Category = General)
	bool bAutoRestrictSelf = true;

	/** If true, the sandbox will attempt to operate in editor builds. This may be useful for debugging an uncooked game build but many features of the editor itself will not work. You must restart the editor instance for changes to take effect. */
	UPROPERTY(EditAnywhere, config, Category = General)
	bool bAllowSandboxInEditor = false;

	/** If true, the game will be reduced to low integrity level after restricting itself. This means most files and registry keys will become read only in addition to other limitations. See Microsoft documentation for details. */
	UPROPERTY(EditAnywhere, config, Category = Windows)
	bool bUseLowIntegrityLevel = true;

	/** If true, the game will be prevented from loading low integrity level libraries after restricting itself. This blocks loading of any executable files written to disk by a low integrity level process. */
	UPROPERTY(EditAnywhere, config, Category = Windows)
	bool bDisallowLowIntegrityLibraries = true;
	
	/** If true, the game will be prevented from spawning child processes after restricting itself. Currently not compatible with Unreal's default crash reporter. */
	UPROPERTY(EditAnywhere, config, Category = Windows)
	bool bDisallowChildProcesses = false;

	/** If true, the game will be prevented from performing system operations that are usually unnecessary like logging the user out of Windows after restricting itself. This unfortunately also blocks clipboard paste from other applications. */
	UPROPERTY(EditAnywhere, config, Category = Windows)
	bool bDisallowSystemOperations = false;
};
